{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "AISupport",
	"Description" : "A simple plugin that makes sure your project loads AIModule and NavigationSystem at runtime",
	"Category" : "AI",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
        {
            "Name": "AISupportModule",
            "Type": "Runtime",
            "LoadingPhase": "PostConfigInit"
        }
	]
}