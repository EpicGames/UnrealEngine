// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AISupportModule : ModuleRules
	{
		public AISupportModule(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
			new string[] {
			}
			);

			PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"NavigationSystem",
				"AIModule"
			}
			);
		}
	}
}
