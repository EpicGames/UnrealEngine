// Copyright Epic Games, Inc. All Rights Reserved.

#include "AISupportModule.h"
#include "AITypes.h"

#define LOCTEXT_NAMESPACE "AISupport"

class FAISupportModule : public IAISupportModule
{
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FAISupportModule, AISupportModule)

void FAISupportModule::StartupModule()
{
	// We need to actually link in something from the AI module, otherwise DLL dependencies will not get loaded early enough
	// It is NOT safe to actually call startup on the AI module though, as that depends on things that get initialized later in launch
	static int32 ForceLink = FAIResources::GetResourcesCount();
	ForceLink++;
}

void FAISupportModule::ShutdownModule()
{

}

#undef LOCTEXT_NAMESPACE
